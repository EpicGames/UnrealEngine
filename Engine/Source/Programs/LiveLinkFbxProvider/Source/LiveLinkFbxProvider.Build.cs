// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class LiveLinkFbxProvider : ModuleRules
{
	public LiveLinkFbxProvider(ReadOnlyTargetRules Target) : base(Target)
	{
		IWYUSupport = IWYUSupport.None;
		bUseRTTI = false;

		// Unreal dependency modules
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"ApplicationCore",
			"Core",
			"CoreUObject",
			"Projects",
			"LiveLinkInterface",
			"LiveLinkMessageBusFramework",
			"Messaging",
			"UdpMessaging",
		});

		PrivateIncludePathModuleNames.AddRange(new string[] {
			"Launch",
		});

		AddEngineThirdPartyPrivateStaticDependencies(Target, "FBX");
	}
}
