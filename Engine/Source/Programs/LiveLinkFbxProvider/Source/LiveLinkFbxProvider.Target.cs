// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;
using System.IO;

public class LiveLinkFbxProviderTarget : TargetRules
{
	public LiveLinkFbxProviderTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;

		DefaultBuildSettings = BuildSettingsVersion.Latest;
		IncludeOrderVersion = EngineIncludeOrderVersion.Latest;

		SolutionDirectory = "Programs/LiveLink";
		AdditionalPlugins.Add("UdpMessaging");

		LaunchModuleName = "LiveLinkFbxProvider";

		// This app compiles against Core/CoreUObject, but not the Engine or Editor
		bCompileAgainstCoreUObject = true;
		bCompileAgainstEngine = false;
		bBuildDeveloperTools = false;
		bCompileWithPluginSupport = false;

		// We use UHT metadata reflection for documenting command line arguments.
		bBuildWithEditorOnlyData = true;

		// This app is a console application (sets entry point to main(), instead of WinMain())
		bIsBuildingConsoleApplication = true;
	}
}
