// Copyright Epic Games, Inc. All Rights Reserved.

#include "RequiredProgramMainCPPInclude.h"

#include "LiveLinkFbxProviderLoop.h"

IMPLEMENT_APPLICATION(LiveLinkFbxProvider, "LiveLinkFbxProvider");

/**
 * Application entry point
 *
 * @param	ArgC	Command-line argument count
 * @param	ArgV	Argument strings
 */
INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	FLiveLinkFbxProviderLoopInitArgs LoopInitArgs(ArgC, ArgV);

	return FLiveLinkFbxProviderLoop(LoopInitArgs).Run(ArgC, ArgV);
}
