// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFbxProviderLoop.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Ticker.h"
#include "Interfaces/IPluginManager.h"
#include "LaunchEngineLoop.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DateTime.h"
#include "Misc/ExpressionParserTypes.h" // for FExpressionError
#include "Misc/OutputDeviceConsole.h"
#include "Modules/ModuleManager.h"
#include "Roles/LiveLinkAnimationRole.h"
#include "Roles/LiveLinkAnimationTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"


#pragma pack(push,8)

THIRD_PARTY_INCLUDES_START
#include <fbxsdk.h>
THIRD_PARTY_INCLUDES_END

#pragma pack(pop)


DEFINE_LOG_CATEGORY_STATIC(LogLiveLinkFbxProvider, Verbose, All);


FLiveLinkFbxProviderLoopInitArgs::FLiveLinkFbxProviderLoopInitArgs(int32 InArgC, TCHAR* InArgV[])
	: ArgC(InArgC)
	, ArgV(InArgV)
{
	FString CmdLine = FCommandLine::BuildFromArgV(nullptr, ArgC, ArgV, nullptr);
	FCommandLine::Set(*CmdLine);

	TArray<FString> PositionalArgs;
	TMap<FString, FString> SwitchArgs;

	{
		const TCHAR* CmdLinePos = *CmdLine;
		FString NextToken;
		while (FParse::Token(CmdLinePos, NextToken, false))
		{
			if (!NextToken.IsEmpty() && (**NextToken == TCHAR('-')))
			{
				FString SwitchName;
				FString SwitchValue;
				if (NextToken.Split(TEXT("="), &SwitchName, &SwitchValue))
				{
					SwitchName.ToLowerInline();
					SwitchArgs.FindOrAdd(MoveTemp(SwitchName), MoveTemp(SwitchValue));
				}
				else
				{
					NextToken.ToLowerInline();
					SwitchArgs.FindOrAdd(MoveTemp(NextToken), FString());
				}
			}
			else
			{
				PositionalArgs.Add(MoveTemp(NextToken));
			}
		}
	}

	if (PositionalArgs.Num() > 0)
	{
		FbxFile = PositionalArgs[0];
	}

	if (const FString* SourceNameArg = SwitchArgs.Find("-sourcename"))
	{
		SourceName = *SourceNameArg;
	}

	if (const FString* TimecodeJamSyncArg = SwitchArgs.Find("-timecodejamsync"))
	{
		bTimecodeJamSync = true;
	}

	if (const FString* FrameRateArg = SwitchArgs.Find("-framerate"))
	{
		TValueOrError<FFrameRate, FExpressionError> ParsedRate = ParseFrameRate(**FrameRateArg);
		if (ParsedRate.HasValue())
		{
			FrameRate = ParsedRate.GetValue();
		}
		else if (ParsedRate.HasError())
		{
			UE_LOGFMT(LogLiveLinkFbxProvider, Error, "Error parsing -FrameRate= argument: %s",
				*ParsedRate.GetError().Text.ToString());
		}
	}
}

FString FLiveLinkFbxProviderLoopInitArgs::GetUsage() const
{
	TStringBuilder<2048> UsageMsg;

	UsageMsg.Appendf(TEXT("Usage:\n"));
	UsageMsg.Appendf(TEXT("\t%s \"...\\path\\to\\file.fbx\"\n\n"),
		ArgC >= 1 ? ArgV[0] : TEXT("LiveLinkFbxProvider"));

	UsageMsg.Appendf(TEXT("Options:\n"));

	const UScriptStruct* InitStruct = StaticStruct();
	for (TFieldIterator<FProperty> Field(InitStruct); Field; ++Field)
	{
		FProperty* Property = *Field;
		
		FString PropertyName = Property->GetName();
		if (Property->IsA<FBoolProperty>() && PropertyName.StartsWith(TEXT("b")))
		{
			PropertyName.RightChopInline(1);
		}

		static const FName TooltipMetaDataName("ToolTip");
		UsageMsg.Appendf(TEXT("\t-%s: %s\n"), *PropertyName, *Property->GetMetaData(TooltipMetaDataName));
	}

	return FString(UsageMsg);
}


FLiveLinkFbxProviderLoop::FLiveLinkFbxProviderLoop(const FLiveLinkFbxProviderLoopInitArgs& InInitArgs)
	: InitArgs(InInitArgs)
{
}


int32 FLiveLinkFbxProviderLoop::Run(int32 ArgC, TCHAR** ArgV)
{
	// Validate the init settings
	checkf(!InitArgs.SourceName.IsEmpty(), TEXT("Source name cannot be empty!"));

	int32 Result = GEngineLoop.PreInit(ArgC, ArgV, TEXT(" -Messaging"));
	if (Result != 0)
	{
		UE_LOGFMT(LogLiveLinkFbxProvider, Error, "FEngineLoop::PreInit failed with result {Result}", Result);
		return Result;
	}

	check(GConfig && GConfig->IsReadyForUse());

	GLogConsole->Show(true);

	ProcessNewlyLoadedUObjects();
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();
	FModuleManager::Get().LoadModule(TEXT("UdpMessaging"));

	// Load internal Concert plugins in the pre-default phase
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PreDefault);

	// Load Concert Sync plugins in default phase
	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::Default);

	IPluginManager::Get().LoadModulesForEnabledPlugins(ELoadingPhase::PostDefault);

	ON_SCOPE_EXIT
	{
		// Allow the game thread to finish processing any latent tasks.
		// They will be relying on what we are going to shutdown...
		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		FEngineLoop::AppPreExit();

		// Unloading Modules isn't handled by AppExit
		FModuleManager::Get().UnloadModulesAtShutdown();

		FEngineLoop::AppExit();
	};

	// Install graceful termination handler, this handles graceful CTRL+C shutdown, 
	// but not CTRL+CLOSE, which will potentially still exit process before the main thread exits.
	// Double CTRL+C signal will also cause process to terminate.
	FPlatformMisc::SetGracefulTerminationHandler();

	if (InitArgs.ArgC < 2)
	{
		UE_LOGFMT(LogLiveLinkFbxProvider, Error, "You must specify an FBX file to load.\n\n{Usage}",
			InitArgs.GetUsage());
		return 1;
	}

	UE_LOGFMT(LogLiveLinkFbxProvider, Display,
		"This example provider will stream a single skeleton from the default FBX scene/take via Message Bus. "
		"Any other data in the FBX file will be ignored.");

	if (!ReadFbx(InitArgs.FbxFile))
	{
		UE_LOGFMT(LogLiveLinkFbxProvider, Error, "Error loading FBX file (\"{FbxFile}\")", InitArgs.FbxFile);
		return 1;
	}

	StartProvider();
	ON_SCOPE_EXIT { StopProvider(); };

	double PrevFrameStart = FPlatformTime::Seconds();
	LoopStartTime = PrevFrameStart;

	if (FirstFrameTime)
	{
		const FFrameTime LoopStartSystemFrameTime = GetSystemFrameTimeWithOffset(0.0);
		FrameTimeOffset = FirstFrameTime.GetValue().Time - LoopStartSystemFrameTime;
	}

	while (!IsEngineExitRequested())
	{
		const double FrameStart = FPlatformTime::Seconds();
		const double FrameDeltaTime = FrameStart - PrevFrameStart;
		PrevFrameStart = FrameStart;

		const double NextFrameTime = Tick();

		FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);

		// Pump & Tick objects
		FTSTicker::GetCoreTicker().Tick(FrameDeltaTime);

		GFrameCounter++;
		GLog->FlushThreadedLogs();

		// Run garbage collection for the UObjects for the rest of the frame or at least to 2 ms
		IncrementalPurgeGarbage(true, FMath::Max<float>(0.002f, NextFrameTime - FPlatformTime::Seconds()));

		// Throttle main thread main fps by sleeping if we still have time
		FPlatformProcess::Sleep(FMath::Max<float>(0.0f, NextFrameTime - FPlatformTime::Seconds()));
	}

	return Result;
}


FLiveLinkFbxSubject::FLiveLinkFbxSubject(FName InSubjectName)
	: SubjectName(InSubjectName)
{

}


bool FLiveLinkFbxSubject::InitializeFromFbxNode(fbxsdk::FbxNode* InSkelNode)
{
	RecursiveGatherBones(InSkelNode);
	return BoneNodes.Num() > 0;
}


void FLiveLinkFbxSubject::RecursiveGatherBones(fbxsdk::FbxNode* InNode, int32 InParentIdx /* = -1 */)
{
	check(InNode);

	if (!InNode->GetNodeAttribute() || InNode->GetNodeAttribute()->GetAttributeType() != FbxNodeAttribute::eSkeleton)
	{
		return;
	}

	BoneNames.Emplace(UTF8_TO_TCHAR(InNode->GetName()));
	BoneParents.Emplace(InParentIdx);
	const int32 ThisIdx = BoneNodes.Emplace(InNode);

	for (int32 ChildIdx = 0; ChildIdx < InNode->GetChildCount(); ++ChildIdx)
	{
		RecursiveGatherBones(InNode->GetChild(ChildIdx), ThisIdx);
	}
}


void FLiveLinkFbxProviderLoop::RecursiveGatherSkeletons(FbxNode* InNode)
{
	check(InNode);

	if (InNode->GetNodeAttribute() && InNode->GetNodeAttribute()->GetAttributeType() == FbxNodeAttribute::eSkeleton)
	{
		const FName SubjectName = FName(TEXT("FBX Subject"), Subjects.Num());
		TSharedRef<FLiveLinkFbxSubject> Subject = MakeShared<FLiveLinkFbxSubject>(SubjectName);
		if (ensure(Subject->InitializeFromFbxNode(InNode)))
		{
			Subjects.Emplace(Subject);
		}
		return;
	}

	for (int32 ChildIdx = 0; ChildIdx < InNode->GetChildCount(); ++ChildIdx)
	{
		RecursiveGatherSkeletons(InNode->GetChild(ChildIdx));
	}
}


struct FLiveLinkFbxTimecodeHelper
{
	FLiveLinkFbxTimecodeHelper(const FLiveLinkFbxSubject& InSubject)
	{
		check(InSubject.GetBoneNodes().Num() > 0);

		// First search ancestors from the root up.
		if (FbxNode* TryRootNode = InSubject.GetBoneNodes()[0])
		{
			while (TryRootNode)
			{
				if (TryGetTimecodeProperties(TryRootNode))
				{
					bHasTimecode = true;
					return;
				}

				TryRootNode = TryRootNode->GetParent();
			}
		}

		// Fall back to searching every other bone.
		for (FbxNode* BoneNode : InSubject.GetBoneNodes())
		{
			if (TryGetTimecodeProperties(BoneNode))
			{
				bHasTimecode = true;
				return;
			}
		}
	}


	bool TryGetTimecodeProperties(FbxNode* InNode)
	{
		bool bFoundAnyTcProperty = false;
		FbxProperty Prop = InNode->GetFirstProperty();
		while (Prop.IsValid())
		{
			UE_LOGFMT(LogLiveLinkFbxProvider, VeryVerbose,
				"Node: {NodeName} - Found prop {PropName}",
				InNode->GetName(), Prop.GetNameAsCStr());

			if (Prop.GetName().CompareNoCase("TCHour") == 0)
			{
				HourProp = Prop;
				bFoundAnyTcProperty = true;
			}
			else if (Prop.GetName().CompareNoCase("TCMinute") == 0)
			{
				MinuteProp = Prop;
				bFoundAnyTcProperty = true;
			}
			else if (Prop.GetName().CompareNoCase("TCSecond") == 0)
			{
				SecondProp = Prop;
				bFoundAnyTcProperty = true;
			}
			else if (Prop.GetName().CompareNoCase("TCFrame") == 0)
			{
				FrameProp = Prop;
				bFoundAnyTcProperty = true;
			}
			else if (Prop.GetName().CompareNoCase("TCSubframe") == 0)
			{
				SubframeProp = Prop;
				bFoundAnyTcProperty = true;
			}

			Prop = InNode->GetNextProperty(Prop);
		}

		return bFoundAnyTcProperty;
	}


	FTimecode TryEvaluateTimecode(FbxAnimEvaluator* InEvaluator, FbxTime InTime)
	{
		FTimecode Timecode;

		if (HourProp.IsValid())
		{
			const int32 HoursValue = FMath::RoundToInt32(InEvaluator->GetPropertyValue<double>(HourProp, InTime));
			Timecode.Hours = FMath::Max(0, HoursValue);
		}

		if (MinuteProp.IsValid())
		{
			const int32 MinutesValue = FMath::RoundToInt32(InEvaluator->GetPropertyValue<double>(MinuteProp, InTime));
			Timecode.Minutes = FMath::Max(0, MinutesValue);
		}

		if (SecondProp.IsValid())
		{
			const int32 SecondsValue = FMath::RoundToInt32(InEvaluator->GetPropertyValue<double>(SecondProp, InTime));
			Timecode.Seconds = FMath::Max(0, SecondsValue);
		}

		if (FrameProp.IsValid())
		{
			const int32 FramesValue = FMath::RoundToInt32(InEvaluator->GetPropertyValue<double>(FrameProp, InTime));
			Timecode.Frames = FMath::Max(0, FramesValue);
		}

		if (SubframeProp.IsValid())
		{
			const double SubframeValue = InEvaluator->GetPropertyValue<double>(SubframeProp, InTime);
			Timecode.Subframe = FMath::Max(0.0, SubframeValue);
		}

		return Timecode;
	}


	bool bHasTimecode = false;
	FbxProperty HourProp;
	FbxProperty MinuteProp;
	FbxProperty SecondProp;
	FbxProperty FrameProp;
	FbxProperty SubframeProp;
};


void FLiveLinkFbxProviderLoop::SnapSubframesToCleanFractions(int32 InSubframeDivisor)
{
	if (InSubframeDivisor <= 1)
	{
		return;
	}

	UE_LOGF(LogLiveLinkFbxProvider, Log, "Snapping subframes (divisor: %d)", InSubframeDivisor);

	const float Step = 100.0f / static_cast<float>(InSubframeDivisor);
	for (FLiveLinkFbxFrame& Frame : Frames)
	{
		if (Frame.Timecode)
		{
			const int32 Residual = FMath::RoundToInt32(Frame.Timecode->Subframe * 100.0f);
			const int32 NearestIdx = FMath::RoundToInt32(static_cast<float>(Residual) / Step);

			if (NearestIdx >= InSubframeDivisor)
			{
				Frame.Timecode->Subframe = 0.0f;
				Frame.Timecode->Frames++;
			}
			else
			{
				Frame.Timecode->Subframe = static_cast<float>(NearestIdx) / static_cast<float>(InSubframeDivisor);
			}
		}
	}
}


bool FLiveLinkFbxProviderLoop::ReadFbx(const FString& InFbxFilename)
{
	if (InFbxFilename.IsEmpty())
	{
		UE_LOGFMT(LogLiveLinkFbxProvider, Error, "No FBX file path specified.");
		UE_LOGFMT(LogLiveLinkFbxProvider, Display, "{Usage}", InitArgs.GetUsage());
		return false;
	}

	if (!FPaths::FileExists(InFbxFilename))
	{
		UE_LOGFMT(LogLiveLinkFbxProvider, Error, "Specified FBX file does not exist (\"{FbxFile}\")", InFbxFilename);
		return false;
	}

	FbxManager* Manager = FbxManager::Create();
	ON_SCOPE_EXIT { Manager->Destroy(); };

	FbxIOSettings* IoSettings = FbxIOSettings::Create(Manager, IOSROOT);
	IoSettings->SetBoolProp(IMP_RELAXED_FBX_CHECK, true);
	Manager->SetIOSettings(IoSettings);

	FbxImporter* Importer = FbxImporter::Create(Manager, "");
	ON_SCOPE_EXIT { Importer->Destroy(); };

	// TStringConversion appends a null terminator.
	if (!ensure(Importer->Initialize(TCHAR_TO_UTF8(*InFbxFilename), -1, Manager->GetIOSettings())))
	{
		return false;
	}

	FbxScene* Scene = FbxScene::Create(Manager, "");
	ON_SCOPE_EXIT { Scene->Destroy(true); };
	if (!ensure(Importer->Import(Scene)))
	{
		return false;
	}

	FbxAxisSystem UnrealAxisSystem(FbxAxisSystem::eZAxis, FbxAxisSystem::eParityOdd, FbxAxisSystem::eLeftHanded);
	UnrealAxisSystem.DeepConvertScene(Scene);

	FbxAnimEvaluator* Evaluator = Scene->GetAnimationEvaluator();

	const FbxTime::EMode TimeMode = Scene->GetGlobalSettings().GetTimeMode();
	if (InitArgs.FrameRate)
	{
		FrameRate = InitArgs.FrameRate.GetValue();
	}
	else
	{
		switch (TimeMode)
		{
			case FbxTime::EMode::eFrames120: FrameRate = FFrameRate(120, 1); break;
			case FbxTime::EMode::eFrames100: FrameRate = FFrameRate(100, 1); break;
			case FbxTime::EMode::eFrames60: FrameRate = FFrameRate(60, 1); break;
			case FbxTime::EMode::eFrames50: FrameRate = FFrameRate(50, 1); break;
			case FbxTime::EMode::eFrames48: FrameRate = FFrameRate(48, 1); break;
			case FbxTime::EMode::eFrames30: FrameRate = FFrameRate(30, 1); break;
			case FbxTime::EMode::eFrames30Drop:
				FrameRate = FFrameRate(30, 1);
				bDropFrame = true;
				break;
			case FbxTime::EMode::eNTSCDropFrame:
				FrameRate = FFrameRate(30'000, 1'001);
				bDropFrame = true;
				break;
			case FbxTime::EMode::eNTSCFullFrame: FrameRate = FFrameRate(30'000, 1'001); break;
			case FbxTime::EMode::ePAL: FrameRate = FFrameRate(25, 1); break;
			case FbxTime::EMode::eFrames24: FrameRate = FFrameRate(24, 1); break;
			default:
				UE_LOGFMT(LogLiveLinkFbxProvider, Error,
					"Unsupported FbxTime::EMode: {TimeMode}; specify the frame rate manually",
					TimeMode);
				return false;
		}
	}

	const FbxTime NominalFramePeriod(FbxTime::GetOneFrameValue(TimeMode));
	NominalFramePeriodSec = NominalFramePeriod.GetSecondDouble();

	if (NominalFramePeriodSec > SMALL_NUMBER)
	{
		UE_LOGF(LogLiveLinkFbxProvider, Log, "FBX native frame period: %.3f ms (%.3f fps)",
			NominalFramePeriodSec * 1000.0, 1.0 / NominalFramePeriodSec);
	}
	else
	{
		UE_LOGF(LogLiveLinkFbxProvider, Warning, "FBX native frame period: %.3f ms (invalid, cannot compute fps)",
			NominalFramePeriodSec * 1000.0);
	}

	// TODO: Support other "takes"?
	FbxAnimStack* AnimStack = Scene->GetSrcObject<FbxAnimStack>(0);
	if (!ensure(AnimStack))
	{
		UE_LOGFMT(LogLiveLinkFbxProvider, Error, "AnimStack 0 not valid");
		return false;
	}

	const FbxTimeSpan AnimSpan = AnimStack->GetLocalTimeSpan();
	const FbxTime AnimStart = AnimSpan.GetStart();
	const FbxTime AnimEnd = AnimSpan.GetStop();
	const FbxTime AnimDuration = AnimEnd - AnimStart;

	FbxNode* RootNode = Scene->GetRootNode();
	if (!ensure(RootNode))
	{
		UE_LOGFMT(LogLiveLinkFbxProvider, Error, "Root node not valid");
		return false;
	}

	RecursiveGatherSkeletons(RootNode);

	TOptional<FLiveLinkFbxTimecodeHelper> TcHelper;
	for (const TSharedRef<FLiveLinkFbxSubject>& Subject : Subjects)
	{
		FLiveLinkFbxTimecodeHelper Helper(*Subject);
		if (Helper.bHasTimecode)
		{
			TcHelper = Helper;
			break;
		}
	}

	FbxAnimCurve* TimecodeCurve = nullptr;

	if (TcHelper)
	{
		FbxProperty TimecodeEvalProp;

		if (TcHelper->SubframeProp.IsValid())
		{
			TimecodeEvalProp = TcHelper->SubframeProp;
		}
		else if (TcHelper->FrameProp.IsValid())
		{
			TimecodeEvalProp = TcHelper->FrameProp;
		}

		if (TimecodeEvalProp.IsValid())
		{
			if (FbxAnimCurveNode* TimecodeCurveNode = TimecodeEvalProp.GetCurveNode(AnimStack))
			{
				TimecodeCurve = TimecodeCurveNode->GetCurve(0);
			}
		}
	}

	int32 LargestTcFrame = 0;
	TOptional<FTimecode> FirstFrameTimecode;

	auto EvalAtTime =
		[this, &AnimStart, &Evaluator, &TcHelper, &LargestTcFrame, &FirstFrameTimecode]
		(FbxTime InEvalTime)
		{
			FLiveLinkFbxFrame& Frame = Frames.Emplace_GetRef();
			Frame.FrameTimeSeconds = (InEvalTime - AnimStart).GetSecondDouble();

			// Extract timecode
			{
				FTimecode Timecode;
				int Field, Residual;
				const bool bTimecodeValid = InEvalTime.GetTime(Timecode.Hours, Timecode.Minutes,
					Timecode.Seconds, Timecode.Frames, Field, Residual);
				if (bTimecodeValid)
				{
					Timecode.Subframe = Residual / 100.0f;
					Timecode.bDropFrameFormat = bDropFrame;
					Frame.Timecode = Timecode;

					LargestTcFrame = FMath::Max(LargestTcFrame, Timecode.Frames);

					if (UNLIKELY(!FirstFrameTimecode))
					{
						FirstFrameTimecode = Timecode;
					}

					UE_LOGF(LogLiveLinkFbxProvider, VeryVerbose, "Seconds: %03.03f - Timecode: %ls",
						Frame.FrameTimeSeconds, *Frame.Timecode->ToString(true, true));
				}
			}

			for (const TSharedRef<FLiveLinkFbxSubject>& Subject : Subjects)
			{
				FLiveLinkFbxSubjectFrame SubjectFrame;
				SubjectFrame.FramePose.Reserve(Subject->GetBoneNodes().Num());
				for (FbxNode* BoneNode : Subject->GetBoneNodes())
				{
					const FbxAMatrix& NodeMatrix = Evaluator->GetNodeLocalTransform(BoneNode, InEvalTime);
					FbxVector4 NodePosition = NodeMatrix.GetT();
					FbxVector4 NodeScale = NodeMatrix.GetS();
					FbxQuaternion NodeRotation = NodeMatrix.GetQ();

					FTransform NodeTransform;
					NodeTransform.SetTranslation(FVector(NodePosition[0], NodePosition[1], NodePosition[2]));
					NodeTransform.SetScale3D(FVector(NodeScale[0], NodeScale[1], NodeScale[2]));
					NodeTransform.SetRotation(FQuat(NodeRotation[0], NodeRotation[1], NodeRotation[2], NodeRotation[3]));
					SubjectFrame.FramePose.Emplace(MoveTemp(NodeTransform));
				}

				Frame.SubjectFrameData.Add(Subject.ToWeakPtr(), MoveTemp(SubjectFrame));
			}
		};

	if (TimecodeCurve)
	{
		// Advance by timecode keys
		UE_LOGFMT(LogLiveLinkFbxProvider, Log, "Using timecode key walk evaluation mode");

		const int KeyCount = TimecodeCurve->KeyGetCount();
		for (int KeyIdx = 0; KeyIdx < KeyCount; ++KeyIdx)
		{
			FbxAnimCurveKey Key = TimecodeCurve->KeyGet(KeyIdx);
			EvalAtTime(Key.GetTime());
		}
	}
	else
	{
		// Advance by nominal frame periods
		UE_LOGFMT(LogLiveLinkFbxProvider, Log, "Using nominal frame period evaluation mode");

		const FbxTime StepPeriod = NominalFramePeriod; // TODO: upsampling?
		for (FbxTime EvalTime = AnimStart; EvalTime <= AnimEnd; EvalTime += StepPeriod)
		{
			EvalAtTime(EvalTime);
		}
	}

	if (!InitArgs.FrameRate)
	{
		if (AnimDuration.GetSecondDouble() >= 1.0)
		{
			if (LargestTcFrame > 0)
			{
				const int32 TcRate = LargestTcFrame + 1;
				FrameRate = FFrameRate(TcRate, 1);
				UE_LOGFMT(LogLiveLinkFbxProvider, Log, "Using detected timecode rate of {FrameRate}",
					FrameRate.ToPrettyText().ToString());
			}
		}
		else
		{
			if (FirstFrameTimecode)
			{
				UE_LOGFMT(LogLiveLinkFbxProvider, Warning,
					"Animation is very short; timecode rate may need to be specified manually with `-FrameRate=`");
			}
		}
	}

	if (NominalFramePeriodSec > 0.0 && LargestTcFrame > 0)
	{
		const double NativeFPS = 1.0 / NominalFramePeriodSec;
		const int32 TcFPS = LargestTcFrame + 1;
		SnapSubframesToCleanFractions(FMath::RoundToInt32(NativeFPS / TcFPS));
	}

	if (FirstFrameTimecode)
	{
		FirstFrameTime = FQualifiedFrameTime(FirstFrameTimecode.GetValue(), FrameRate);
	}

	return ensure(Subjects.Num() > 0 && Frames.Num() > 0);
}


void FLiveLinkFbxProviderLoop::StartProvider()
{
	// Create a connection object
	LiveLinkProvider = ILiveLinkProvider::CreateLiveLinkProvider(InitArgs.SourceName);
	check(LiveLinkProvider);

	UE_LOGF(LogLiveLinkFbxProvider, Display, "%ls Initialized", *InitArgs.SourceName);

	for (const TSharedRef<FLiveLinkFbxSubject>& Subject : Subjects)
	{
		FLiveLinkStaticDataStruct SubjectStatic = FLiveLinkStaticDataStruct(FLiveLinkSkeletonStaticData::StaticStruct());
		FLiveLinkSkeletonStaticData* SkeletonStatic = SubjectStatic.Cast<FLiveLinkSkeletonStaticData>();
		SkeletonStatic->BoneNames = Subject->GetBoneNames();
		SkeletonStatic->BoneParents = Subject->GetBoneParents();
		LiveLinkProvider->UpdateSubjectStaticData(Subject->GetSubjectName(), ULiveLinkAnimationRole::StaticClass(), MoveTemp(SubjectStatic));
	}
}


void FLiveLinkFbxProviderLoop::StopProvider()
{
	// tell live link that the subjects don't exist anymore
	for (const TSharedRef<FLiveLinkFbxSubject>& Subject : Subjects)
	{
		LiveLinkProvider->RemoveSubject(Subject->GetSubjectName());
	}
}


double FLiveLinkFbxProviderLoop::Tick()
{
	checkf(NextFrameIdx >= 0 && NextFrameIdx < Frames.Num(),
		TEXT("Invalid next frame index %i (max %i)"),
		NextFrameIdx, Frames.Num());

	FLiveLinkFbxFrame* Frame = &Frames[NextFrameIdx];
	while ((LoopStartTime + Frame->FrameTimeSeconds) < FPlatformTime::Seconds())
	{
		for (const FLiveLinkFbxFrame::FSubjectPair& SubjectPair : Frame->SubjectFrameData)
		{
			TSharedPtr<FLiveLinkFbxSubject> Subject = SubjectPair.Key.Pin();
			if (!ensure(Subject))
			{
				continue;
			}

			const FLiveLinkFbxSubjectFrame& SubjectFrame = SubjectPair.Value;

			FLiveLinkFrameDataStruct FrameData = FLiveLinkFrameDataStruct(FLiveLinkAnimationFrameData::StaticStruct());
			FLiveLinkAnimationFrameData* AnimFrameData = FrameData.Cast<FLiveLinkAnimationFrameData>();

			AnimFrameData->WorldTime = FLiveLinkWorldTime(FPlatformTime::Seconds());
			AnimFrameData->Transforms = SubjectFrame.FramePose;

			if (Frame->Timecode)
			{
				FQualifiedFrameTime SceneTime(*Frame->Timecode, FrameRate);
				if (InitArgs.bTimecodeJamSync)
				{
					SceneTime.Time -= FrameTimeOffset;
				}
				AnimFrameData->MetaData.SceneTime = SceneTime;

				const double Now = FPlatformTime::Seconds();

				static double LastFrameSendTime = Now;
				const double IntraFrameSendDelta = Now - LastFrameSendTime;
				LastFrameSendTime = Now;

				static FQualifiedFrameTime LastFrameTime = SceneTime;
				const double IntraFrameTcDelta = SceneTime.AsSeconds() - LastFrameTime.AsSeconds();
				LastFrameTime = SceneTime;

				const double TimeSinceLoopStart = Now - LoopStartTime;
				const double LoopDelta = TimeSinceLoopStart - Frame->FrameTimeSeconds;
				UE_LOGF(LogLiveLinkFbxProvider, VeryVerbose, "Sending frame %i (%ls) (intra-tick Δ %.3f ms) (TC Δ %.3f ms)",
					NextFrameIdx, *Frame->Timecode->ToString(true, true), IntraFrameSendDelta * 1000.0, IntraFrameTcDelta * 1000.0);
			}

			LiveLinkProvider->UpdateSubjectFrameData(Subject->GetSubjectName(), MoveTemp(FrameData));
		}

		if (++NextFrameIdx >= Frames.Num())
		{
			// Handle playback looping.
			NextFrameIdx = 0;
			LoopStartTime = LoopStartTime + Frame->FrameTimeSeconds + NominalFramePeriodSec;
			if (FirstFrameTime)
			{
				const FFrameTime LoopStartSystemFrameTime = GetSystemFrameTimeWithOffset(NominalFramePeriodSec);
				FrameTimeOffset = FirstFrameTime.GetValue().Time - LoopStartSystemFrameTime;
			}
		}

		Frame = &Frames[NextFrameIdx];
	}

	return LoopStartTime + Frame->FrameTimeSeconds;
}


FFrameTime FLiveLinkFbxProviderLoop::GetSystemFrameTimeWithOffset(double InOffsetSec) const
{
	const FDateTime DateTime = FDateTime::Now();
	const FTimespan Timespan = DateTime.GetTimeOfDay();
	return FrameRate.AsFrameTime(Timespan.GetTotalSeconds() + InOffsetSec);
}
