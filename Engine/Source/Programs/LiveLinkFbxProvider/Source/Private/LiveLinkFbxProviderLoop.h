// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LiveLinkProvider.h"
#include "Misc/FrameRate.h"
#include "LiveLinkFbxProviderLoop.generated.h"


namespace fbxsdk
{
	class FbxNode;
}


USTRUCT()
struct FLiveLinkFbxProviderLoopInitArgs
{
	GENERATED_BODY()

	FLiveLinkFbxProviderLoopInitArgs() {}
	FLiveLinkFbxProviderLoopInitArgs(int32 ArgC, TCHAR* ArgV[]);

	/** Path to FBX file to stream. */
	FString FbxFile;

	/** Name to use for the provided Live Link source. */
	UPROPERTY()
	FString SourceName = TEXT("LiveLinkFbxProvider");

	/** Frame rate override. */
	UPROPERTY()
	TOptional<FFrameRate> FrameRate;

	/** Rebase the timecode each playback loop to roughly align it to system time. */
	UPROPERTY()
	bool bTimecodeJamSync = false;

	int32 ArgC = 0;
	TCHAR** ArgV = nullptr;

	FString GetUsage() const;
};


class FLiveLinkFbxSubject
{
public:
	FLiveLinkFbxSubject(FName InSubjectName);

	bool InitializeFromFbxNode(fbxsdk::FbxNode* InSkelNode);

	FName GetSubjectName() const { return SubjectName; }
	const TArray<FName>& GetBoneNames() const { return BoneNames; }
	const TArray<int32>& GetBoneParents() const { return BoneParents; }
	const TArray<fbxsdk::FbxNode*>& GetBoneNodes() const { return BoneNodes; }

private:
	void RecursiveGatherBones(fbxsdk::FbxNode* InNode, int32 InParentIdx = -1);

private:
	FName SubjectName;
	TArray<FName> BoneNames;
	TArray<int32> BoneParents;
	TArray<fbxsdk::FbxNode*> BoneNodes;
};


struct FLiveLinkFbxSubjectFrame
{
	TArray<FTransform> FramePose;
};


struct FLiveLinkFbxFrame
{
	double FrameTimeSeconds;
	TOptional<FTimecode> Timecode;

	using FSubjectPair = TPair<TWeakPtr<FLiveLinkFbxSubject>, FLiveLinkFbxSubjectFrame>;
	TMap<TWeakPtr<FLiveLinkFbxSubject>, FLiveLinkFbxSubjectFrame> SubjectFrameData;
};


/**
 * Blocking main loop for running the provider application.
 */
struct FLiveLinkFbxProviderLoop
{
	FLiveLinkFbxProviderLoop(const FLiveLinkFbxProviderLoopInitArgs& InitArgs);

	int32 Run(int32 ArgC, TCHAR** ArgV);

private:
	bool ReadFbx(const FString& InFbxFilename);
	void RecursiveGatherSkeletons(fbxsdk::FbxNode* InNode);
	void SnapSubframesToCleanFractions(int32 InSubframeDivisor);

	void StartProvider();
	void StopProvider();

	double Tick();

	FFrameTime GetSystemFrameTimeWithOffset(double InDeltaSec) const;

private:
	const FLiveLinkFbxProviderLoopInitArgs InitArgs;

	TSharedPtr<ILiveLinkProvider> LiveLinkProvider;

	TArray<TSharedRef<FLiveLinkFbxSubject>> Subjects;
	TArray<FLiveLinkFbxFrame> Frames;

	FFrameRate FrameRate;
	bool bDropFrame = false;
	double NominalFramePeriodSec = 0.0;
	TOptional<FQualifiedFrameTime> FirstFrameTime;
	double LoopStartTime = 0.0;
	FFrameTime FrameTimeOffset;
	int32 NextFrameIdx = 0;
};
